// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class HAPMedia : ModuleRules
	{
		public HAPMedia(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
                    "Projects",
					"WmfMediaFactory",
					"WmfMedia",
                });

            if (Target.Platform == UnrealTargetPlatform.Win64)
            {
                PrivateDependencyModuleNames.AddRange(
                    new string[] {
                    "HAPLib",
                    "SnappyLib",
                    });

                PublicDelayLoadDLLs.Add("mf.dll");
				PublicDelayLoadDLLs.Add("mfplat.dll");
				PublicDelayLoadDLLs.Add("mfplay.dll");
				PublicDelayLoadDLLs.Add("shlwapi.dll");

                PublicSystemLibraries.Add("mf.lib");
                PublicSystemLibraries.Add("mfplat.lib");
                PublicSystemLibraries.Add("mfuuid.lib");
                PublicSystemLibraries.Add("shlwapi.lib");
                PublicSystemLibraries.Add("d3d11.lib");
				// engine must explicitly use the bundled compiler library to make shader compilation repeatable
				//PublicSystemLibraries.Add("d3dcompiler.lib");
			}
		}
	}
}
