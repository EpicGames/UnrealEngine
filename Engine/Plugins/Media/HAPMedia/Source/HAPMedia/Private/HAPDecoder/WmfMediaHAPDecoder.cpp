// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaHAPDecoder.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "WmfMediaCommon.h"

#include "hap.h"

#include "HAPMediaModule.h"

#include "GenericPlatform/GenericPlatformAtomics.h"
#include "Misc/Paths.h"
#include "Windows/AllowWindowsPlatformTypes.h"

void MyHapDecodeCallback(HapDecodeWorkFunction InFunction, void* InParameter, unsigned int InCount, void* InInfo)
{
	unsigned int i;
	for (i = 0; i < InCount; i++)
	{
		InFunction(InParameter, i);
	}
}

const GUID DecoderGUID_HAP = { 0x48617031, 0x767A, 0x494D, { 0xB4, 0x78, 0xF2, 0x9D, 0x25, 0xDC, 0x90, 0x37 } };
const GUID DecoderGUID_HAP_ALPHA = { 0x48617035, 0x767A, 0x494D, { 0xB4, 0x78, 0xF2, 0x9D, 0x25, 0xDC, 0x90, 0x37 } };
const GUID DecoderGUID_HAP_Q = { 0x48617059, 0x767A, 0x494D, { 0xB4, 0x78, 0xF2, 0x9D, 0x25, 0xDC, 0x90, 0x37 } };
const GUID DecoderGUID_HAP_Q_ALPHA = { 0x4861704D, 0x767A, 0x494D, { 0xB4, 0x78, 0xF2, 0x9D, 0x25, 0xDC, 0x90, 0x37 } };

bool WmfMediaHAPDecoder::IsSupported(const GUID& InGuid)
{
	return ((InGuid == DecoderGUID_HAP) ||
			(InGuid == DecoderGUID_HAP_ALPHA) ||
			(InGuid == DecoderGUID_HAP_Q) ||
			(InGuid == DecoderGUID_HAP_Q_ALPHA));
}


bool WmfMediaHAPDecoder::SetOutputFormat(const GUID& InGuid, GUID& OutVideoFormat)
{

	if ((InGuid == DecoderGUID_HAP) || (InGuid == DecoderGUID_HAP_Q))
	{
		OutVideoFormat =  MFVideoFormat_NV12;
		return true;
	}
	else if ((InGuid == DecoderGUID_HAP_ALPHA) || (InGuid == DecoderGUID_HAP_Q_ALPHA))
	{
		OutVideoFormat = MFVideoFormat_ARGB32;
		return true;
	}
	else
	{
		return false;
	}
}

WmfMediaHAPDecoder::WmfMediaHAPDecoder()
	: WmfMediaDecoder(),
	InputSubType{ 0 }
{
}


WmfMediaHAPDecoder::~WmfMediaHAPDecoder()
{
}


#pragma warning(push)
#pragma warning(disable:4838)
HRESULT WmfMediaHAPDecoder::QueryInterface(REFIID riid, void** ppv)
{
	static const QITAB qit[] = 
	{
		QITABENT(WmfMediaHAPDecoder, IMFTransform),
		{ 0 }
	};

	return QISearch(this, qit, riid, ppv);
}
#pragma warning(pop)



HRESULT WmfMediaHAPDecoder::GetOutputAvailableType(DWORD dwOutputStreamID, DWORD dwTypeIndex, IMFMediaType** ppType)
{
	if (ppType == NULL)
	{
		return E_INVALIDARG;
	}

	if (dwOutputStreamID != 0)
	{
		return MF_E_INVALIDSTREAMNUMBER;
	}

	if (dwTypeIndex != 0)
	{
		return MF_E_NO_MORE_TYPES;
	}

	FScopeLock Lock(&CriticalSection);

	HRESULT hr = S_OK;

	TComPtr<IMFMediaType> TempOutputType;

	if (InputType == NULL)
	{
		hr = MF_E_TRANSFORM_TYPE_NOT_SET;
	}

	if (SUCCEEDED(hr))
	{
		hr = MFCreateMediaType(&TempOutputType);
	}

	if (SUCCEEDED(hr))
	{
		hr = TempOutputType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Video);
	}

	if (SUCCEEDED(hr))
	{
		check(bIsExternalBufferEnabled);
		hr = TempOutputType->SetGUID(MF_MT_SUBTYPE, MFVideoFormat_NV12);
		if (SUCCEEDED(hr))
		{
			if (InputSubType == DecoderGUID_HAP)
			{
				hr = TempOutputType->SetGUID(UE_WMF_PrivateFormatGUID, MFVideoFormat_L8);
			}
			else if (InputSubType == DecoderGUID_HAP_ALPHA)
			{
				hr = TempOutputType->SetGUID(UE_WMF_PrivateFormatGUID, MFVideoFormat_L16);
			}
			else if (InputSubType == DecoderGUID_HAP_Q)
			{
				hr = TempOutputType->SetGUID(UE_WMF_PrivateFormatGUID, MFVideoFormat_RGB8);
			}
			else
			{
				hr = TempOutputType->SetGUID(UE_WMF_PrivateFormatGUID, MFVideoFormat_D16);
			}
		}
	}

	if (SUCCEEDED(hr))
	{
		hr = TempOutputType->SetUINT32(MF_MT_FIXED_SIZE_SAMPLES, TRUE);
	}

	if (SUCCEEDED(hr))
	{
		hr = TempOutputType->SetUINT32(MF_MT_ALL_SAMPLES_INDEPENDENT, TRUE);
	}

	if (SUCCEEDED(hr))
	{
		hr = TempOutputType->SetUINT32(MF_MT_SAMPLE_SIZE, OutputImageSize);
	}

	if (SUCCEEDED(hr))
	{
		hr = MFSetAttributeSize(TempOutputType, MF_MT_FRAME_SIZE, ImageWidthInPixels, ImageHeightInPixels);
	}

	if (SUCCEEDED(hr))
	{
		hr = MFSetAttributeRatio(TempOutputType, MF_MT_FRAME_RATE, FrameRate.Numerator, FrameRate.Denominator);
	}

	if (SUCCEEDED(hr))
	{
		hr = TempOutputType->SetUINT32(MF_MT_INTERLACE_MODE, MFVideoInterlace_Progressive);
	}

	if (SUCCEEDED(hr))
	{
		hr = MFSetAttributeRatio(TempOutputType, MF_MT_PIXEL_ASPECT_RATIO, 1, 1);
	}

	if (SUCCEEDED(hr))
	{
		*ppType = TempOutputType;
		(*ppType)->AddRef();
	}

	return hr;
}



HRESULT WmfMediaHAPDecoder::GetAttributes(IMFAttributes** ppAttributes)
{
	TComPtr<IMFAttributes> Attributes;
	HRESULT hr = MFCreateAttributes(&Attributes, 1);

	if (SUCCEEDED(hr))
	{
		const GUID NoneEmptyDummyGUID = { 0x86c8d2ec, 0xa1e, 0x4637, {0xbe, 0x7a, 0x7f, 0xea, 0x0, 0xb2, 0xb3, 0xb1} };
		Attributes->SetUINT32(NoneEmptyDummyGUID, 0U);
		*ppAttributes = Attributes;
		(*ppAttributes)->AddRef();
		return S_OK;
	}
	else
	{
		return E_NOTIMPL;
	}
}


HRESULT WmfMediaHAPDecoder::ProcessMessage(MFT_MESSAGE_TYPE eMessage, ULONG_PTR ulParam)
{
	FScopeLock Lock(&CriticalSection);

	HRESULT hr = S_OK;

	switch (eMessage)
	{
	case MFT_MESSAGE_COMMAND_FLUSH:
		hr = OnFlush();
		break;

	case MFT_MESSAGE_COMMAND_DRAIN:
	case MFT_MESSAGE_NOTIFY_BEGIN_STREAMING:
		hr = OnDiscontinuity();
		break;

	default:
		break;
	}

	return hr;
}


HRESULT WmfMediaHAPDecoder::ProcessOutput(DWORD dwFlags, DWORD cOutputBufferCount, MFT_OUTPUT_DATA_BUFFER* pOutputSamples, DWORD* pdwStatus)
{
	if (dwFlags != 0)
	{
		return E_INVALIDARG;
	}

	if (pOutputSamples == NULL || pdwStatus == NULL)
	{
		return E_POINTER;
	}

	if (cOutputBufferCount != 1)
	{
		return E_INVALIDARG;
	}

	if (OutputQueue.IsEmpty())
	{
		return MF_E_TRANSFORM_NEED_MORE_INPUT;
	}

	FScopeLock Lock(&CriticalSection);

	HRESULT hr = S_OK;

	if (SUCCEEDED(hr))
	{
		TComPtr<IMFSample> Sample;

		MFCreateSample(&Sample);

		pOutputSamples[0].pSample = Sample.Get();
		Sample->AddRef();

		// If used with DX11 we need a (dummy, but amply sized!) memory buffer attached to the sample to make WMF happy
		if (!MediaBuffer.IsValid())
		{
			MFCreateMemoryBuffer(OutputImageSize, &MediaBuffer);
			MediaBuffer->SetCurrentLength(OutputImageSize);
		}

		Sample->AddBuffer(MediaBuffer);
	}

	if (SUCCEEDED(hr))
	{
		hr = InternalProcessOutput(pOutputSamples[0].pSample);
	}

	pOutputSamples[0].dwStatus = 0;
	*pdwStatus = 0;

	return hr;
}


bool WmfMediaHAPDecoder::IsExternalBufferSupported() const
{
	return true;
}


HRESULT WmfMediaHAPDecoder::InternalProcessOutput(IMFSample* InSample)
{
	if (OutputQueue.IsEmpty())
	{
		return MF_E_TRANSFORM_NEED_MORE_INPUT;
	}

	DataBuffer OuputDataBuffer;
	OutputQueue.Dequeue(OuputDataBuffer);

	LONGLONG TimeStamp = OuputDataBuffer.TimeStamp;

	// Are we using external buffers?
	check(bIsExternalBufferEnabled);

	// Get buffer.
	int32 BufferSize = ImageWidthInPixels * ImageHeightInPixels;
	int32 AlphaBufferSize = 0;

	// HapQ Alpha has a separate alpha buffer.
	if (InputSubType == DecoderGUID_HAP_Q_ALPHA)
	{
		AlphaBufferSize = BufferSize / 2;
	}

	// Hap is 4 bits per pixel.
	if (InputSubType == DecoderGUID_HAP)
	{
		BufferSize = BufferSize / 2;
	}

	// Get external buffer.
	TArray<uint8>* pExternalBuffer = AllocateExternalBuffer(TimeStamp, BufferSize + AlphaBufferSize);
	if (pExternalBuffer == nullptr)
	{
		return 0;
	}

	// Copy to buffer.
	pExternalBuffer->SetNum(BufferSize + AlphaBufferSize);
	FMemory::Memcpy(pExternalBuffer->GetData(), OuputDataBuffer.Color.GetData(), BufferSize);

	if (AlphaBufferSize > 0)
	{
		FMemory::Memcpy(pExternalBuffer->GetData() + BufferSize, OuputDataBuffer.Alpha.GetData(), AlphaBufferSize);
	}
	
	InputQueue.Enqueue(MoveTemp(OuputDataBuffer));

	HRESULT hr = InSample->SetUINT32(MFSampleExtension_CleanPoint, TRUE);

	if (SUCCEEDED(hr))
	{
		hr = InSample->SetSampleTime(TimeStamp);
	}

	if (SUCCEEDED(hr))
	{
		hr = InSample->SetSampleDuration(SampleDuration);
	}

	return hr;
}


HRESULT WmfMediaHAPDecoder::OnCheckInputType(IMFMediaType* InMediaType)
{
	HRESULT hr = S_OK;

	if (InputType)
	{
		DWORD dwFlags = 0;
		if (S_OK == InputType->IsEqual(InMediaType, &dwFlags))
		{
			return S_OK;
		}
		else
		{
			return MF_E_INVALIDTYPE;
		}
	}

	GUID MajorType = { 0 };
	GUID SubType = { 0 };
	UINT32 Width = 0;
	UINT32 Height = 0;
	MFRatio Fps = { 0 };

	hr = InMediaType->GetMajorType(&MajorType);

	if (SUCCEEDED(hr))
	{
		if (MajorType != MFMediaType_Video)
		{
			hr = MF_E_INVALIDTYPE;
		}
	}

	if (SUCCEEDED(hr))
	{
		hr = InMediaType->GetGUID(MF_MT_SUBTYPE, &SubType);
	}

	if (SUCCEEDED(hr))
	{
		if (SubType != DecoderGUID_HAP &&
			SubType != DecoderGUID_HAP_ALPHA &&
			SubType != DecoderGUID_HAP_Q &&
			SubType != DecoderGUID_HAP_Q_ALPHA)
		{
			hr = MF_E_INVALIDTYPE;
		}

	}

	if (SUCCEEDED(hr))
	{
		hr = MFGetAttributeSize(InMediaType, MF_MT_FRAME_SIZE, &Width, &Height);
	}

	if (SUCCEEDED(hr))
	{
		hr = MFGetAttributeRatio(InMediaType, MF_MT_FRAME_RATE, (UINT32*)&Fps.Numerator, (UINT32*)&Fps.Denominator);
	}

	return hr;
}


HRESULT WmfMediaHAPDecoder::OnSetInputType(IMFMediaType* InMediaType)
{
	HRESULT hr = S_OK;

	hr = MFGetAttributeSize(InMediaType, MF_MT_FRAME_SIZE, &ImageWidthInPixels, &ImageHeightInPixels);

	if (SUCCEEDED(hr))
	{
		hr = MFGetAttributeRatio(InMediaType, MF_MT_FRAME_RATE, (UINT32*)&FrameRate.Numerator, (UINT32*)&FrameRate.Denominator);
	}

	if (SUCCEEDED(hr))
	{
		hr = MFFrameRateToAverageTimePerFrame(
			FrameRate.Numerator, 
			FrameRate.Denominator, 
			&SampleDuration
			);
	}

	if (SUCCEEDED(hr))
	{
		check(bIsExternalBufferEnabled);
		OutputImageSize = InputImageSize = ImageWidthInPixels * ImageHeightInPixels * 4;

		InputType = InMediaType;
		InputType->AddRef();

		InMediaType->GetGUID(MF_MT_SUBTYPE, &InputSubType);
	}

	return hr;
}


bool WmfMediaHAPDecoder::HasPendingOutput() const
{
	return !InputQueue.IsEmpty();
}


HRESULT WmfMediaHAPDecoder::InternalProcessInput(LONGLONG InTimeStamp, BYTE* InData, DWORD InDataSize)
{
	check(InData)

	unsigned OutputTextureCount = 0;
	unsigned int Result = HapGetFrameTextureCount(InData, InDataSize, &OutputTextureCount);

	int ChunkCount = 0;
	Result = HapGetFrameTextureChunkCount(InData, InDataSize, 0, &ChunkCount);

	unsigned int ColorOutputBufferTextureFormat = 0;
	unsigned long ColorOutputBufferBytesUsed = 0;

	unsigned int AlphaOutputBufferTextureFormat = 0;
	unsigned long AlphaOutputBufferBytesUsed = 0;

	DataBuffer InputDataBuffer;

	if (!InputQueue.IsEmpty())
	{
		InputQueue.Dequeue(InputDataBuffer);
	}
	else
	{
		InputDataBuffer.Color.AddUninitialized(InputImageSize);
		if (InputSubType == DecoderGUID_HAP_Q_ALPHA)
		{
			InputDataBuffer.Alpha.AddUninitialized(InputImageSize);
		}
	}
		
	Result = HapDecode(InData, InDataSize, 0, MyHapDecodeCallback, nullptr, InputDataBuffer.Color.GetData(), InputImageSize, &ColorOutputBufferBytesUsed, &ColorOutputBufferTextureFormat);

	if (OutputTextureCount == 2 && InputSubType == DecoderGUID_HAP_Q_ALPHA)
	{
		Result = HapDecode(InData, InDataSize, 1, MyHapDecodeCallback, nullptr, InputDataBuffer.Alpha.GetData(), InputImageSize, &AlphaOutputBufferBytesUsed, &AlphaOutputBufferTextureFormat);
	}

	if (Result == HapResult_No_Error)
	{
		InputDataBuffer.TimeStamp = InTimeStamp;
		OutputQueue.Enqueue(MoveTemp(InputDataBuffer));
	}

	return S_OK;
}



#include "Windows/HideWindowsPlatformTypes.h"

#endif // WMFMEDIA_SUPPORTED_PLATFORM
