// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAPMediaModule.h"

#include "HAPDecoder/WmfMediaHAPDecoder.h"
#include "IWmfMediaModule.h"
#include "WmfMediaCodec/WmfMediaCodecGenerator.h"
#include "WmfMediaCodec/WmfMediaCodecManager.h"

DEFINE_LOG_CATEGORY(LogHAPMedia);

#define LOCTEXT_NAMESPACE "FWmfMediaCodecModule"

class FHAPMediaModule : public IModuleInterface
{
public:

	FHAPMediaModule()
	{ }

public:


public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
#if WMFMEDIA_SUPPORTED_PLATFORM

		// initialize Windows Media Foundation
		HRESULT Result = MFStartup(MF_VERSION);

		if (FAILED(Result))
		{
			UE_LOGF(LogHAPMedia, Log, "Failed to initialize Windows Media Foundation, Error %i", Result);

			return;
		}

		if (IWmfMediaModule* Module = IWmfMediaModule::Get())
		{
			if (Module->IsInitialized())
			{
				Module->GetCodecManager()->AddCodec(MakeUnique<WmfMediaCodecGenerator<WmfMediaHAPDecoder>>(true));
			}
		}

#endif //WMFMEDIA_SUPPORTED_PLATFORM

	}

	virtual void ShutdownModule() override
	{
#if WMFMEDIA_SUPPORTED_PLATFORM
		// shutdown Windows Media Foundation
		MFShutdown();
#endif // WMFMEDIA_SUPPORTED_PLATFORM
	}

	// Codec could still be in use
	virtual bool SupportsDynamicReloading()
	{
		return false;
	}

protected:

private:

	/** Whether the module has been initialized. */
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FHAPMediaModule, HAPMedia);
